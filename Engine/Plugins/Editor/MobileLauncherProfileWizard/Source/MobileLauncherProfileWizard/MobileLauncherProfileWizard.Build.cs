// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MobileLauncherProfileWizard : ModuleRules
	{
		public MobileLauncherProfileWizard(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
                    "Core",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
                    "LauncherServices",
                    "CoreUObject",
                    "TargetPlatform",
                    "DesktopPlatform",
                    "Json",
                    "Slate",
                    "SlateCore",
                    "InputCore",
                    "AppFramework",
                    "Projects",
                    // ... add private dependencies that you statically link with here ...
				}
                );
		}
	}
}
