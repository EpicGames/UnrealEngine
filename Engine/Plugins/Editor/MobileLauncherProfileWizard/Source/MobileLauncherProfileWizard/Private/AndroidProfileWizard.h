// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ILauncherProfileManager.h"

class FAndroidProfileWizard : public ILauncherProfileWizard
{
public:
	virtual FText GetName() const override;
	virtual FText GetDescription() const override;
	virtual void HandleCreateLauncherProfile(const ILauncherProfileManagerRef& ProfileManager) override;
};

