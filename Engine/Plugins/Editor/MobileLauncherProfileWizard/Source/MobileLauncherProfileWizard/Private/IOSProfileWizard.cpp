// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSProfileWizard.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "SProfileWizardUI.h"

#define LOCTEXT_NAMESPACE "MobileLauncherProfileWizard"

FText FIOSProfileWizard::GetName() const
{
	return LOCTEXT("IOSWizardName", "Minimal IOS App + DLC...");
}

FText FIOSProfileWizard::GetDescription() const
{
	return LOCTEXT("IOSWizardDescription", "This wizard will help to create two launcher profiles, one for minimal IOS App and another for downloadable content.");
}

namespace IOSProfileConstants
{
	const FString AppPlatformName(TEXT("IOS"));
	const FString AppReleaseName(TEXT("1.0"));
	const FString DLCName(TEXT("DLC1.0"));
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

static void SetupIOSAppProfile(ILauncherProfileRef& AppProfile, const FProfileParameters& Params, const FString& ProjectPath)
{
	AppProfile->SetProjectSpecified(true);
	AppProfile->SetProjectPath(ProjectPath);

	AppProfile->SetBuildUAT(true);
	// App build configuration
	AppProfile->SetBuildMode(ELauncherProfileBuildModes::Auto);
	AppProfile->SetBuildConfiguration(Params.BuildConfiguration);
	
	//// Cooking
	AppProfile->SetCookMode(ELauncherProfileCookModes::ByTheBook);
	AppProfile->SetCookConfiguration(Params.BuildConfiguration);
	for (const FString& MapName : Params.AppMaps)
	{
		AppProfile->AddCookedMap(MapName);
	}
	AppProfile->AddCookedPlatform(IOSProfileConstants::AppPlatformName);

	// Release settings
	AppProfile->SetCreateReleaseVersion(true);
	AppProfile->SetCreateReleaseVersionName(IOSProfileConstants::AppReleaseName);
	AppProfile->SetIncrementalCooking(false);
	AppProfile->SetCompressed(false);
	AppProfile->SetDeployWithUnrealPak(true);

	// Packaging
	AppProfile->SetPackagingMode(ELauncherProfilePackagingModes::Locally);

	// Archive
	AppProfile->SetArchive(true);
	{
		FString AppDir = Params.ArchiveDirectory / TEXT("App/") / IOSProfileConstants::AppReleaseName;
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		
		if (!PlatformFile.DirectoryExists(*AppDir))
		{
			PlatformFile.CreateDirectoryTree(*AppDir);
		}
		
		AppProfile->SetArchiveDirectory(AppDir);
	}
	
	// Deploy
	AppProfile->SetDeploymentMode(ELauncherProfileDeploymentModes::DoNotDeploy);

	// Launch
	AppProfile->SetLaunchMode(ELauncherProfileLaunchModes::DoNotLaunch);
}

static void SetupIOSDLCProfile(ILauncherProfileRef& DLCProfile, const FProfileParameters& Params, const FString& ProjectPath)
{
	DLCProfile->SetProjectSpecified(true);
	DLCProfile->SetProjectPath(ProjectPath);

	DLCProfile->SetBuildUAT(true);
	// App build configuration
	DLCProfile->SetBuildMode(ELauncherProfileBuildModes::DoNotBuild);
	DLCProfile->SetBuildConfiguration(Params.BuildConfiguration);
	
	//// Cooking
	DLCProfile->SetCookMode(ELauncherProfileCookModes::ByTheBook);
	DLCProfile->SetCookConfiguration(Params.BuildConfiguration);
	for (const FString& MapName : Params.DLCMaps)
	{
		DLCProfile->AddCookedMap(MapName);
	}
	
	DLCProfile->AddCookedPlatform(IOSProfileConstants::AppPlatformName);
	
	// Release settings
	DLCProfile->SetCreateReleaseVersion(false);
	DLCProfile->SetBasedOnReleaseVersionName(IOSProfileConstants::AppReleaseName);
	DLCProfile->SetCreateDLC(true);
	DLCProfile->SetDLCName(IOSProfileConstants::DLCName);
	DLCProfile->SetDLCIncludeEngineContent(true);
		
	DLCProfile->SetIncrementalCooking(false);
	DLCProfile->SetCompressed(false);
	DLCProfile->SetDeployWithUnrealPak(true);

	// HTTP chunk data
	DLCProfile->SetGenerateHttpChunkData(true);
	DLCProfile->SetHttpChunkDataReleaseName(IOSProfileConstants::DLCName);
	{
		FString CloudDir = Params.ArchiveDirectory / TEXT("HTTPchunks/") / IOSProfileConstants::DLCName;
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		
		if (!PlatformFile.DirectoryExists(*CloudDir))
		{
			PlatformFile.CreateDirectoryTree(*CloudDir);
		}
		
		DLCProfile->SetHttpChunkDataDirectory(CloudDir);
	}

	// Packaging
	DLCProfile->SetPackagingMode(ELauncherProfilePackagingModes::DoNotPackage);
	
	// Deploy
	DLCProfile->SetDeploymentMode(ELauncherProfileDeploymentModes::DoNotDeploy);

	// Launch
	DLCProfile->SetLaunchMode(ELauncherProfileLaunchModes::DoNotLaunch);
}

static void CreateIOSProfiles(const FProfileParameters& Params, FString ProjectPath, ILauncherProfileManagerRef ProfileManager)
{
	const FString ProjectName = FPaths::GetBaseFilename(ProjectPath);
	const FString AppProfileName = ProjectName + TEXT(" - IOS App");
	const FString DLCProfileName = ProjectName + TEXT(" - IOS DLC");

	// Make profile names unique
	FString AppProfileNameUnique = AppProfileName;
	FString DLCProfileNameUnique = DLCProfileName;
	int32 UniqueCounter = 1;
	while (ProfileManager->FindProfile(AppProfileNameUnique).IsValid() 
			|| ProfileManager->FindProfile(DLCProfileNameUnique).IsValid())
	{
		AppProfileNameUnique = AppProfileName + FString::FromInt(UniqueCounter);
		DLCProfileNameUnique = DLCProfileName + FString::FromInt(UniqueCounter);
		UniqueCounter++;
	}

	// Create archive directory
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*Params.ArchiveDirectory))
	{
		PlatformFile.CreateDirectoryTree(*Params.ArchiveDirectory);
	}
	
	// Add App profile
	ILauncherProfileRef AppProfile = ProfileManager->AddNewProfile();
	SetupIOSAppProfile(AppProfile, Params, ProjectPath);
	ProfileManager->ChangeProfileName(AppProfile, AppProfileNameUnique);
	ProfileManager->SaveJSONProfile(AppProfile);

	// Add DLC profile
	ILauncherProfileRef DLCProfile = ProfileManager->AddNewProfile();
	SetupIOSDLCProfile(DLCProfile, Params, ProjectPath);
	ProfileManager->ChangeProfileName(DLCProfile, DLCProfileNameUnique);
	ProfileManager->SaveJSONProfile(DLCProfile);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FIOSProfileWizard::HandleCreateLauncherProfile(const ILauncherProfileManagerRef& ProfileManager)
{
	const FVector2D WindowSize = FVector2D(940, 540);

	FText WindowTitle = LOCTEXT("CreateIOSProfileWizardTitle", "IOS App + DLC");
	FString ProjectPath = ProfileManager->GetProjectPath();
	
	TSharedRef<SWindow> AddProfileWindow =
		SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(WindowSize)
		.SizingRule(ESizingRule::FixedSize)
		.SupportsMinimize(false).SupportsMaximize(false);

	TSharedRef<SProfileWizardUI> ProfilesDialog =
		SNew(SProfileWizardUI)
		.ProfilePlatform(EProfilePlatform::IOS)
		.ProjectPath(ProjectPath)
		.OnCreateProfileEvent(FCreateProfileEvent::CreateStatic(&CreateIOSProfiles, ProjectPath, ProfileManager));

	AddProfileWindow->SetContent(ProfilesDialog);
	FSlateApplication::Get().AddWindow(AddProfileWindow);
}

#undef LOCTEXT_NAMESPACE
