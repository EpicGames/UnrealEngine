// Copyright Epic Games, Inc. All Rights Reserved.

#include "ILauncherServicesModule.h"
#include "IMobileLauncherProfileWizard.h"
#include "AndroidProfileWizard.h"
#include "IOSProfileWizard.h"

#define LOCTEXT_NAMESPACE "MobileLauncherProfileWizard"

class FMobileLauncherProfileWizardModule : public IMobileLauncherProfileWizardModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	void OnProfileManagerInitialized(ILauncherProfileManager& ProfileManager);
private:
	TSharedPtr<FAndroidProfileWizard> AndroidWizardPtr;
	TSharedPtr<FIOSProfileWizard> IOSWizardPtr;
};

IMPLEMENT_MODULE( FMobileLauncherProfileWizardModule, MobileLauncherProfileWizard )

void FMobileLauncherProfileWizardModule::StartupModule()
{
	ILauncherServicesModule::ProfileManagerInitializedDelegate.AddRaw(this, &FMobileLauncherProfileWizardModule::OnProfileManagerInitialized);
	
	// Check if ProfileManager was already initialized
	ILauncherServicesModule* LauncherServicesModule = FModuleManager::GetModulePtr<ILauncherServicesModule>("LauncherServices");
	if (LauncherServicesModule)
	{
		TSharedRef<ILauncherProfileManager> LauncherProfileManager = LauncherServicesModule->GetProfileManager();
		OnProfileManagerInitialized(LauncherProfileManager.Get());
	}
}

void FMobileLauncherProfileWizardModule::ShutdownModule()
{
	ILauncherServicesModule::ProfileManagerInitializedDelegate.RemoveAll(this);
	
	ILauncherServicesModule* LauncherServicesModule = FModuleManager::GetModulePtr<ILauncherServicesModule>("LauncherServices");
	if (LauncherServicesModule)
	{
		TSharedRef<ILauncherProfileManager> LauncherProfileManager = LauncherServicesModule->GetProfileManager();
		LauncherProfileManager->UnregisterProfileWizard(AndroidWizardPtr);
		LauncherProfileManager->UnregisterProfileWizard(IOSWizardPtr);
	}
}

void FMobileLauncherProfileWizardModule::OnProfileManagerInitialized(ILauncherProfileManager& ProfileManager)
{
	AndroidWizardPtr = MakeShareable(new FAndroidProfileWizard());
	ProfileManager.RegisterProfileWizard(AndroidWizardPtr);

	IOSWizardPtr = MakeShareable(new FIOSProfileWizard());
	ProfileManager.RegisterProfileWizard(IOSWizardPtr);
}

#undef LOCTEXT_NAMESPACE
