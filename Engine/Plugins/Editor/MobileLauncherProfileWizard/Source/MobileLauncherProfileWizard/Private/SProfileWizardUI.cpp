// Copyright Epic Games, Inc. All Rights Reserved.


#include "SProfileWizardUI.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SSeparator.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Workflow/SWizard.h"
#include "Widgets/Input/SHyperlink.h"
#include "InstalledPlatformInfo.h"
#include "PlatformInfo.h"
#include "GameProjectHelper.h"

#define LOCTEXT_NAMESPACE "MobileLauncherProfileWizard"

namespace ProfileWizardUI
{
	const static FText PlatformNameIOS(LOCTEXT("PlatformNameIOS", "IOS"));
	const static FText PlatformStoreIOS(LOCTEXT("PlatformStoreIOS", "App Store"));
	const static FText PlatformNameAndroid(LOCTEXT("PlatformNameAndroid", "Android"));
	const static FText PlatformStoreAndroid(LOCTEXT("PlatformStoreAndroid", "Play Store"));
}

SProfileWizardUI::SProfileWizardUI()
	: ProfilePlatform(EProfilePlatform::Android)
	, BuildConfiguration(EBuildConfiguration::Development)
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProfileWizardUI::Construct(const FArguments& InArgs)
{
	ProfilePlatform = InArgs._ProfilePlatform;
	ProjectPath = InArgs._ProjectPath;
	OnCreateProfileEvent = InArgs._OnCreateProfileEvent;
	
	// Cache project data
	CacheProjectMapList();
	CacheCookFlavorsList();
		
	ChildSlot
	[
		SNew(SBorder)
		.Padding(18)
		.BorderImage(FAppStyle::GetBrush("Docking.Tab.ContentAreaBrush"))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			[
				SAssignNew(MainWizard, SWizard)
				.ShowPageList(false)
				.CanFinish(this, &SProfileWizardUI::CanFinish)
				.FinishButtonText(LOCTEXT("FinishButtonText", "Create Profile"))
				.FinishButtonToolTip(LOCTEXT("FinishButtonToolTip", "Creates the launcher profiles for packaging a simple application and downloadable content."))
				.OnCanceled(this, &SProfileWizardUI::CancelClicked)
				.OnFinished(this, &SProfileWizardUI::FinishClicked)
				.PageFooter()
				[
					SNew(SBorder)
					.Visibility(this, &SProfileWizardUI::GetGlobalErrorLabelVisibility)
					.BorderImage(FAppStyle::GetBrush("NewClassDialog.ErrorLabelBorder"))
					.Padding(FMargin(0, 5))
					.Content()
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("MessageLog.Warning"))
						]

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SProfileWizardUI::GetGlobalErrorLabelText)
							.TextStyle(FAppStyle::Get(), "NewClassDialog.ErrorLabelFont")
						]
					]
				]

				//
				// Destination page
				//
				+SWizard::Page()
				.CanShow(true)
				[
					SNew(SVerticalBox)

					// Destination page title
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NewClassDialog.PageTitle")
						.Text(this, &SProfileWizardUI::GetDestinationPageTitleText)
					]

					// Title spacer
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 2, 0, 8)
					[
						SNew(SSeparator)
					]

					// Destination page description
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 10)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.FillWidth(1.f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(this, &SProfileWizardUI::GetDestinationPageDescriptionText)
						]
					]

					// Destination page settings
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						.Padding(8.0)
						[
							SNew(SVerticalBox)
					
							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(LOCTEXT("DestinationDirectoryTitle", "Specify a folder where the resulting executable and Cloud distribution content will be stored:"))
							]
					
							+ SVerticalBox::Slot()
							.AutoHeight()
							.Padding(0.0, 4.0, 0.0, 0.0)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.FillWidth(1.0)
								.Padding(0.0, 0.0, 0.0, 3.0)
								[
									// archive path text box
									SNew(SEditableTextBox)
									.Text(this, &SProfileWizardUI::GetDestinationDirectoryText)
									.OnTextCommitted(this, &SProfileWizardUI::OnDestinationDirectoryTextCommitted)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Right)
								.Padding(4.0, 0.0, 0.0, 0.0)
								[
									// browse button
									SNew(SButton)
										.ContentPadding(FMargin(6.0, 2.0))
										.Text(LOCTEXT("BrowseButtonText", "Browse..."))
										.ToolTipText(LOCTEXT("BrowseButtonToolTip", "Browse for the directory"))
										.OnClicked(this, &SProfileWizardUI::HandleBrowseDestinationButtonClicked)
								]
							]
						]
					]
				]
				
				
				//
				// Distributable App page
				//
				+SWizard::Page()
				.CanShow(this, &SProfileWizardUI::CanShowApplicationPage)
				[
					SNew(SVerticalBox)

					// Application page title
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NewClassDialog.PageTitle")
						.Text(this, &SProfileWizardUI::GetApplicationPageTitleText)
					]

					// Title spacer
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 2, 0, 8)
					[
						SNew(SSeparator)
					]

					// Application page description
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 10)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.FillWidth(1.f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(this, &SProfileWizardUI::GetApplicationPageDescriptionText)
						]
					]

					// Application page build settings
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 10)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
						.Padding(8.0)
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("AppConfigurationComboBoxLabel", "Build Configuration:"))
							]

							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SComboButton)
								.VAlign(VAlign_Center)
								.ButtonContent()
								[
									SNew(STextBlock)
									.Font(FCoreStyle::Get().GetFontStyle(TEXT("NormalFont")))
									.Text(this, &SProfileWizardUI::GetBuildConfigurationSelectorText)
								]
								.ContentPadding(FMargin(4.0f, 2.0f))
								.MenuContent()
								[
									MakeBuildConfigurationMenuContent()
								]
							]
						]
					]
					
					// Application page summary and map list
					+SVerticalBox::Slot()
					.FillHeight(1.0)
					[
						SNew(SHorizontalBox)
						
						// Application page summary
						+ SHorizontalBox::Slot()
						.FillWidth(0.5f)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(this, &SProfileWizardUI::GetApplicationPageSummaryText)
						]
						
						// Application map list
						+ SHorizontalBox::Slot()
						.FillWidth(0.5f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.Padding(8.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("AppCookedMapsLabel", "Choose a map(s) to distribute with an application:"))
								]

								+ SVerticalBox::Slot()
								.FillHeight(1.0)
								.Padding(0.0f, 2.0f, 0.0f, 0.0f)
								[
									SNew(SListView<TSharedPtr<FString>>)
									.ListItemsSource(&ProjectMapList)
									.OnGenerateRow(this, &SProfileWizardUI::HandleMapListViewGenerateRow, EProfileTarget::Application)
									.SelectionMode(ESelectionMode::None)
								]
							]
						]
					]
				]

				//
				// Downloadable content page (DLC)
				//
				+SWizard::Page()
				.CanShow(this, &SProfileWizardUI::CanShowDLCPage)
				[
					SNew(SVerticalBox)

					// DLC page title
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0)
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NewClassDialog.PageTitle")
						.Text(this, &SProfileWizardUI::GetDLCPageTitleText)
					]

					// Title spacer
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 2, 0, 8)
					[
						SNew(SSeparator)
					]

					// DLC page description
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 10)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.FillWidth(1.f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.AutoWrapText(true)
							.Text(this, &SProfileWizardUI::GetDLCPageDescriptionText)
						]
					]

					+ SVerticalBox::Slot()
					.FillHeight(1.0)
					[
						SNew(SHorizontalBox)
						
						// DLC cooked flavors
						+ SHorizontalBox::Slot()
						.Padding(4.0f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.Padding(8.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("DLCCookedPlatformsLabel", "Choose a texture format for downloadable content:"))
								]

								+ SVerticalBox::Slot()
								.FillHeight(1.0)
								.Padding(0.0f, 2.0f, 0.0f, 0.0f)
								[
									SNew(SListView<TSharedPtr<FString>>)
									.ListItemsSource(&DLCFlavorList)
									.OnGenerateRow(this, &SProfileWizardUI::HandleCookFlavorViewGenerateRow)
									.SelectionMode(ESelectionMode::None)
									.IsEnabled(this, &SProfileWizardUI::IsCookFlavorEnabled)
								]
							]
						]

						// DLC maps
						+ SHorizontalBox::Slot()
						.Padding(4.0f)
						[
							SNew(SBorder)
							.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
							.Padding(8.0f)
							[
								SNew(SVerticalBox)

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(STextBlock)
									.Text(LOCTEXT("DLCCookedMapsLabel", "Choose a map(s) to include into downloadable content:"))
								]

								+ SVerticalBox::Slot()
								.FillHeight(1.0)
								.Padding(0.0f, 2.0f, 0.0f, 0.0f)
								[
									SNew(SListView<TSharedPtr<FString>>)
									.ListItemsSource(&ProjectMapList)
									.OnGenerateRow(this, &SProfileWizardUI::HandleMapListViewGenerateRow, EProfileTarget::DLC)
									.SelectionMode(ESelectionMode::None)
								]
								
								// Select All/No maps shortcuts
								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SHorizontalBox)

									+ SHorizontalBox::Slot()
									.FillWidth(1.0f)
									.HAlign(HAlign_Right)
									[
										SNew(STextBlock)
										.Text(LOCTEXT("SelectLabel", "Select:"))
									]

									+ SHorizontalBox::Slot()
									.AutoWidth()
									.Padding(8.0f, 0.0f)
									[
										// all maps link
										SNew(SHyperlink)
										.OnNavigate(this, &SProfileWizardUI::HandleAllMapsButton, true, EProfileTarget::DLC)
										.Text(LOCTEXT("AllDLCCookedMapsButtonLabel", "All"))
										.ToolTipText(LOCTEXT("AllDLCCookedMapsButtonTooltip", "Select all maps."))
									]

									+ SHorizontalBox::Slot()
									.AutoWidth()
									[
										// no maps hyper link
										SNew(SHyperlink)
										.OnNavigate(this, &SProfileWizardUI::HandleAllMapsButton, false, EProfileTarget::DLC)
										.Text(LOCTEXT("NoDLCCookedMapsButtonLabel", "None"))
										.ToolTipText(LOCTEXT("NoDLCCookedMapsButtonTooltip", "Deselect all maps."))
									]
								]
							]
						]
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<SWidget> SProfileWizardUI::MakeBuildConfigurationMenuContent()
{
	struct FConfigInfo
	{
		FText ToolTip;
		EBuildConfiguration Configuration;
	};

	const static FConfigInfo Configurations[] =
	{
		{ LOCTEXT("DebugActionHint", "Debug configuration."), EBuildConfiguration::Debug },
		{ LOCTEXT("DebugGameActionHint", "DebugGame configuration."), EBuildConfiguration::DebugGame },
		{ LOCTEXT("DevelopmentActionHint", "Development configuration."), EBuildConfiguration::Development },
		{ LOCTEXT("ShippingActionHint", "Shipping configuration."), EBuildConfiguration::Shipping },
		{ LOCTEXT("TestActionHint", "Test configuration."), EBuildConfiguration::Test }
	};

	// create build configurations menu
	FMenuBuilder MenuBuilder(true, NULL);
	{
		for (const FConfigInfo& ConfigInfo : Configurations)
		{
			if (FInstalledPlatformInfo::Get().IsValidConfiguration(ConfigInfo.Configuration))
			{
				FUIAction UIAction(FExecuteAction::CreateSP(this, &SProfileWizardUI::HandleBuildConfigurationMenuEntryClicked, ConfigInfo.Configuration));
				MenuBuilder.AddMenuEntry(EBuildConfigurations::ToText(ConfigInfo.Configuration), ConfigInfo.ToolTip, FSlateIcon(), UIAction);
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

void SProfileWizardUI::HandleBuildConfigurationMenuEntryClicked(EBuildConfiguration InConfiguration)
{
	BuildConfiguration = InConfiguration;
}

FText SProfileWizardUI::GetBuildConfigurationSelectorText() const
{
	return EBuildConfigurations::ToText(BuildConfiguration);
}

TSharedRef<ITableRow> SProfileWizardUI::HandleMapListViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable, EProfileTarget::Type InProfileTarget)
{
	return 
		SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		.Style(FAppStyle::Get(), "NewClassDialog.ParentClassListView.TableRow")
		[
			SNew(SCheckBox)
			.IsChecked(this, &SProfileWizardUI::HandleMapListViewCheckBoxIsChecked, InItem, InProfileTarget)
			.OnCheckStateChanged(this, &SProfileWizardUI::HandleMapListViewCheckBoxCheckStateChanged, InItem, InProfileTarget)
			.Padding(FMargin(6.0, 2.0))
			[
				SNew(STextBlock)
				.Text(FText::FromString(*InItem))
			]
		];
}

ECheckBoxState SProfileWizardUI::HandleMapListViewCheckBoxIsChecked(TSharedPtr<FString> InMapName, EProfileTarget::Type InProfileTarget) const
{
	return SelectedMaps[InProfileTarget].Contains(*InMapName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SProfileWizardUI::HandleMapListViewCheckBoxCheckStateChanged(ECheckBoxState NewState, TSharedPtr<FString> InMapName, EProfileTarget::Type InProfileTarget)
{
	if (NewState == ECheckBoxState::Checked)
	{
		SelectedMaps[InProfileTarget].Add(*InMapName);
	}
	else
	{
		SelectedMaps[InProfileTarget].Remove(*InMapName);
	}
}

bool SProfileWizardUI::CanShowApplicationPage() const
{
	return !ArchiveDirectory.IsEmpty();
}

bool SProfileWizardUI::CanShowDLCPage() const
{
	return SelectedMaps[EProfileTarget::Application].Num() > 0;
}

TSharedRef<ITableRow> SProfileWizardUI::HandleCookFlavorViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return 
		SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		.Style(FAppStyle::Get(), "NewClassDialog.ParentClassListView.TableRow")
		[
			SNew(SCheckBox)
			.IsChecked(this, &SProfileWizardUI::HandleCookFlavorViewCheckBoxIsChecked, InItem)
			.OnCheckStateChanged(this, &SProfileWizardUI::HandleCookFlavorViewCheckBoxCheckStateChanged, InItem)
			.Padding(FMargin(6.0, 2.0))
			[
				SNew(STextBlock)
				.Text(FText::FromString(*InItem))
			]
		];
}

ECheckBoxState SProfileWizardUI::HandleCookFlavorViewCheckBoxIsChecked(TSharedPtr<FString> InItem) const
{
	return DLCSelectedFlavors.Contains(*InItem) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SProfileWizardUI::HandleCookFlavorViewCheckBoxCheckStateChanged(ECheckBoxState NewState, TSharedPtr<FString> InItem)
{
	if (NewState == ECheckBoxState::Checked)
	{
		DLCSelectedFlavors.Add(*InItem);
	}
	else
	{
		DLCSelectedFlavors.Remove(*InItem);
	}
}

bool SProfileWizardUI::IsCookFlavorEnabled() const
{
	return (DLCFlavorList.Num() > 1);
}

void SProfileWizardUI::HandleAllMapsButton(bool bSelect,  EProfileTarget::Type InProfileTarget)
{
	if (bSelect)
	{
		for (const TSharedPtr<FString>& MapName : ProjectMapList)
		{
			SelectedMaps[InProfileTarget].Add(*MapName);
		}
	}
	else
	{
		SelectedMaps[InProfileTarget].Empty();
	}
}

FText SProfileWizardUI::GetApplicationPageTitleText() const
{
	if (ProfilePlatform == EProfilePlatform::Android)
	{
		return LOCTEXT("AppPageTitleAndroid", "Distributable APK"); 
	}
	else
	{
		return LOCTEXT("AppPageTitleIOS", "Distributable Application"); 
	}
}

FText SProfileWizardUI::GetApplicationPageDescriptionText() const
{
	return LOCTEXT("AppPageDescription", "Creates a launcher profile for a minimal distributable application");
}

FText SProfileWizardUI::GetApplicationPageSummaryText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("PlatformStore"), ProfilePlatform == EProfilePlatform::Android ? ProfileWizardUI::PlatformStoreAndroid : ProfileWizardUI::PlatformStoreIOS);
	
	return FText::Format(LOCTEXT("AppPageSummary", "Only the contents referenced by the map(s) you choose will be packaged into the application for upload to the {PlatformStore}. Typically you should choose only a single map which contains a Level Blueprint to kick off the BuildPatchServices code, and some UMG user interface to show the download progress and any error conditions."), Args);
}

FText SProfileWizardUI::GetDLCPageTitleText() const
{
	return LOCTEXT("DLCTitle", "Downloadable content");
}
	
FText SProfileWizardUI::GetDLCPageDescriptionText() const
{
	if (ProfilePlatform == EProfilePlatform::Android)
	{
		return LOCTEXT("DLCPageDesciptionAndroid", "Choose the texture formats you wish to support. The user's device will download the content in the most appropriate format for their device. Only the contents referenced by the map(s) you choose will be packaged into DLC.");
	}
	else
	{
		return LOCTEXT("DLCPageDesciptionAndroidIOS", "Only the contents referenced by the map(s) you choose will be packaged into DLC.");
	}
}

EVisibility SProfileWizardUI::GetNameErrorLabelVisibility() const
{
	return GetNameErrorLabelText().IsEmpty() ? EVisibility::Hidden : EVisibility::Visible;
}

FText SProfileWizardUI::GetNameErrorLabelText() const
{
	return FText::GetEmpty();
}

EVisibility SProfileWizardUI::GetGlobalErrorLabelVisibility() const
{
	return GetGlobalErrorLabelText().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SProfileWizardUI::GetGlobalErrorLabelText() const
{
	return FText::GetEmpty();
}

void SProfileWizardUI::CancelClicked()
{
	CloseContainingWindow();
}

bool SProfileWizardUI::CanFinish() const
{
	const bool bMapsSelected = (SelectedMaps[EProfileTarget::Application].Num() > 0 && SelectedMaps[EProfileTarget::DLC].Num() > 0);
	const bool bCookFlavorSelected = (DLCSelectedFlavors.Num() > 0);
	
	return bMapsSelected && bCookFlavorSelected;
}

void SProfileWizardUI::FinishClicked()
{
	check(CanFinish());

	FProfileParameters Parameters;
	Parameters.BuildConfiguration = BuildConfiguration;
	Parameters.ArchiveDirectory = ArchiveDirectory;
	Parameters.AppMaps = SelectedMaps[EProfileTarget::Application].Array();
	Parameters.DLCMaps = SelectedMaps[EProfileTarget::DLC].Array();
	Parameters.DLCCookFlavors = DLCSelectedFlavors.Array();
		
	OnCreateProfileEvent.Execute(Parameters);

	CloseContainingWindow();
}

void SProfileWizardUI::CacheProjectMapList()
{
	ProjectMapList.Reset();

	FString BaseProjectPath = FPaths::GetPath(ProjectPath);
	TArray<FString> AvailableMaps = FGameProjectHelper::GetAvailableMaps(BaseProjectPath, false, true);

	for (const FString& MapName : AvailableMaps)
	{
		ProjectMapList.Add(MakeShareable(new FString(MapName)));
	}
}

void SProfileWizardUI::CacheCookFlavorsList()
{
	using namespace PlatformInfo;
	
	static const FName AndroidVanillaName = FName("Android");
	static const FName IOSVanillaName = FName("IOS");
	const FName& TargetPlatformName = (ProfilePlatform == EProfilePlatform::Android ? AndroidVanillaName : IOSVanillaName);
		
	DLCFlavorList.Empty();

	const PlatformInfo::FTargetPlatformInfo* Info = FindPlatformInfo(TargetPlatformName);
	for (const FTargetPlatformInfo* PlatformFlaworInfo : Info->Flavors)
	{
		DLCFlavorList.Add(MakeShareable(new FString(PlatformFlaworInfo->Name.ToString())));
	}

	// In case no flavors, add vanilla name as cooking target
	if (DLCFlavorList.Num() == 0)
	{
		DLCFlavorList.Add(MakeShareable(new FString(TargetPlatformName.ToString())));
	}

	// If only one target, make it selected by default
	if (DLCFlavorList.Num() == 1)
	{
		DLCSelectedFlavors.Add(*DLCFlavorList[0].Get());
	}
}

void SProfileWizardUI::CloseContainingWindow()
{
	TSharedPtr<SWindow> ContainingWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());

	if (ContainingWindow.IsValid())
	{
		ContainingWindow->RequestDestroyWindow();
	}
}

FText SProfileWizardUI::GetDestinationPageTitleText() const
{
	if (ProfilePlatform == EProfilePlatform::Android)
	{
		return LOCTEXT("AndroidDestinationPageTitle", "Minimal Android APK + DLC");
	}
	else
	{
		return LOCTEXT("IOSDestinationPageTitle", "Minimal IOS App + DLC");
	}
}

FText SProfileWizardUI::GetDestinationPageDescriptionText() const
{
	const static FText PlatformDLCIOS(LOCTEXT("PlatformDLCIOS", "The second profile packages the latest content and divides it to chunks files which should be uploaded to the Cloud"));
	const static FText PlatformDLCAndroid(LOCTEXT("PlatformDLCAndroid", "The second profile packages the latest content for each texture format you want to support. The packaged data is divided into chunk files which should be uploaded to the Cloud"));

	FFormatNamedArguments Args;
	Args.Add(TEXT("PlatformName"), ProfilePlatform == EProfilePlatform::Android ? ProfileWizardUI::PlatformNameAndroid : ProfileWizardUI::PlatformNameIOS);
	Args.Add(TEXT("PlatformStore"), ProfilePlatform == EProfilePlatform::Android ? ProfileWizardUI::PlatformStoreAndroid : ProfileWizardUI::PlatformStoreIOS);
	Args.Add(TEXT("PlatformDLC"), ProfilePlatform == EProfilePlatform::Android ? PlatformDLCAndroid : PlatformDLCIOS);
	
	return FText::Format(LOCTEXT("DestinationPageDescription", "This wizard will create two Project Launcher profiles designed to help you package your {PlatformName} game so that the majority of the game's contents, and future content updates, can distributed to your users via HTTP from a Cloud provider or Content Distribution Network when the user launches your app. The first profile is used to generate a small {PlatformName} executable for distribution to the {PlatformStore}. This executable will contain Unreal Engine and your game code, but only the minimum assets required to display the download user interface to the user, while they download your game's latest content to their device. {PlatformDLC}.\n\nThe executable needs code added to use the BuildPatchServices module to download the only the necessary chunks to the user's device and reconstruct the contents file before launching your game."), 
		Args);
}

FText SProfileWizardUI::GetDestinationDirectoryText() const
{
	return FText::FromString(ArchiveDirectory);
}

void SProfileWizardUI::OnDestinationDirectoryTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	ArchiveDirectory = InText.ToString();
}
		
FReply SProfileWizardUI::HandleBrowseDestinationButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString FolderPath;
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			ParentWindowHandle,
			LOCTEXT("FolderDialogTitle", "Choose a directory").ToString(),
			ArchiveDirectory,
			FolderPath
			);

		if (bFolderSelected)
		{
			if (!FolderPath.EndsWith(TEXT("/")))
			{
				FolderPath += TEXT("/");
			}

			ArchiveDirectory = FolderPath;
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
