// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

namespace ETextCommit { enum Type : int; }

class ITableRow;
class STableViewBase;
class SWizard;
enum class ECheckBoxState : uint8;

namespace EProfileTarget
{
	enum Type
	{
		Application,
		DLC,
	
		Num
	};
};

namespace EProfilePlatform
{
	enum Type
	{
		Android,
		IOS,

		Num
	};
};

struct FProfileParameters
{
	EBuildConfiguration BuildConfiguration;
	FString	ArchiveDirectory;
	TArray<FString>	AppMaps;
	TArray<FString>	DLCMaps;
	TArray<FString>	DLCCookFlavors;
};

DECLARE_DELEGATE_OneParam(FCreateProfileEvent, const FProfileParameters& /*FProfileParameters*/);

class SProfileWizardUI : public SCompoundWidget
{
public:
	SProfileWizardUI();

	SLATE_BEGIN_ARGS(SProfileWizardUI)
	{}

	///** Platform we target */
	SLATE_ARGUMENT(EProfilePlatform::Type, ProfilePlatform)

	///** The full path to a project */
	SLATE_ARGUMENT(FString, ProjectPath)

	///** Event called when user want to create profile */
	SLATE_EVENT( FCreateProfileEvent, OnCreateProfileEvent )
	
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

private:
	/** Generate menu content for build configuration widget */
	TSharedRef<SWidget> MakeBuildConfigurationMenuContent();
	
	/** Handle build configuration selection */
	void HandleBuildConfigurationMenuEntryClicked(EBuildConfiguration Configuration);
	
	/** Get a text for a selected build configuration */
	FText GetBuildConfigurationSelectorText() const;

	/** Generate a widget for specified map */
	TSharedRef<ITableRow> HandleMapListViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable, EProfileTarget::Type InProfileTarget);
	
	/** Whether specified map is selected */
	ECheckBoxState HandleMapListViewCheckBoxIsChecked(TSharedPtr<FString> InMapName, EProfileTarget::Type InProfileTarget) const;
	
	/** Handle specified map state */
	void HandleMapListViewCheckBoxCheckStateChanged(ECheckBoxState NewState, TSharedPtr<FString> InMapName, EProfileTarget::Type InProfileTarget);

	/** Generate a widget for specified cook flavor */
	TSharedRef<ITableRow> HandleCookFlavorViewGenerateRow(TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	
	/** Whether specified cook flavor is selected */
	ECheckBoxState HandleCookFlavorViewCheckBoxIsChecked(TSharedPtr<FString> InItem) const;
	
	/** Handle specified cook flavor state */
	void HandleCookFlavorViewCheckBoxCheckStateChanged(ECheckBoxState NewState, TSharedPtr<FString> InMapName);
	
	/** Whether cook flavor list widget in enabled */
	bool IsCookFlavorEnabled() const;

	/** Handle select all/no maps button */
	void HandleAllMapsButton(bool bSelect,  EProfileTarget::Type InProfileTarget);

	/** Whether wizard can proceed to Application page */
	bool CanShowApplicationPage() const;

	/** Whether wizard can proceed to DLC page */
	bool CanShowDLCPage() const;

	/** Gets the Title text for application page */
	FText GetApplicationPageTitleText() const;
	
	/** Gets the Desciption text for application page */
	FText GetApplicationPageDescriptionText() const;
	
	/** Gets the Summary text for application page */
	FText GetApplicationPageSummaryText() const;

	/** Gets the Title text for DLC page */
	FText GetDLCPageTitleText() const;
	
	/** Gets the Desciption text for DLC page */
	FText GetDLCPageDescriptionText() const;

	/** Gets the visibility of the name error label */
	EVisibility GetNameErrorLabelVisibility() const;

	/** Gets the text to display in the name error label */
	FText GetNameErrorLabelText() const;

	/** Gets the visibility of the global error label */
	EVisibility GetGlobalErrorLabelVisibility() const;

	/** Gets the text to display in the global error label */
	FText GetGlobalErrorLabelText() const;

	/** Handler for when cancel is clicked */
	void CancelClicked();

	/** Returns true if Finish is allowed */
	bool CanFinish() const;

	/** Handler for when finish is clicked */
	void FinishClicked();

	/** Get archive directory text */
	FText GetDestinationDirectoryText() const;
		
	/** Handle browse button for archive directory*/
	FReply HandleBrowseDestinationButtonClicked();

	/** Handle commit event for archive directory text */
	void OnDestinationDirectoryTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

	/** Get destination page title text */
	FText GetDestinationPageTitleText() const;

	/** Get destination page description text */
	FText GetDestinationPageDescriptionText() const;
			
private:
	/** Cache project avaiable maps */
	void CacheProjectMapList();
	
	/** Cache project avaiable cook flavors */
	void CacheCookFlavorsList();

	/** Closes the window that contains this widget */
	void CloseContainingWindow();

private:
	/** The wizard widget */
	TSharedPtr<SWizard> MainWizard;

	/** Platform we target: Android or IOS */
	EProfilePlatform::Type ProfilePlatform;

	/** Full path to project we target */
	FString ProjectPath;

	/** Selected project build configuration */
	EBuildConfiguration BuildConfiguration;

	/** Cached list of project maps */
	TArray<TSharedPtr<FString>> ProjectMapList;
	
	/** Selected maps for each profile: Application and DLC */
	TSet<FString> SelectedMaps[EProfileTarget::Num];

	/** Available cook flavors for target platform DLC */
	TArray<TSharedPtr<FString>> DLCFlavorList;

	/** Selected cook flavors for target platform DLC */
	TSet<FString> DLCSelectedFlavors;

	/** Directory where to store build products */
	FString ArchiveDirectory;
	
	/** Event on CreateProfile action */
	FCreateProfileEvent OnCreateProfileEvent;
};
